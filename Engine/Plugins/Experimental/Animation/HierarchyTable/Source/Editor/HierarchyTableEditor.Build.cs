// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class HierarchyTableEditor : ModuleRules
{
	public HierarchyTableEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "HTEd";

		PrivateDependencyModuleNames.AddAll(
			"AssetDefinition",
			"Core",
			"CoreUObject",
			"Engine", 
			"InputCore",
			"Slate",
			"SlateCore",
			"UnrealEd",
			"HierarchyTableRuntime",
			"ToolMenus",
			"ToolWidgets"
		);
	}
}
