// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultHierarchyTableTypeHandler.h"
#include "DefaultHierarchyTableType.h"
#include "HierarchyTable.h"
#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultHierarchyTableTypeHandler)

#define LOCTEXT_NAMESPACE "UHierarchyTable_TableTypeHandler_Default"

void UHierarchyTable_TableTypeHandler_Default::ConstructHierarchy()
{
	// Do nothing, hierarchy is built manually
}

bool UHierarchyTable_TableTypeHandler_Default::FactoryConfigureProperties(FInstancedStruct& TableType) const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
