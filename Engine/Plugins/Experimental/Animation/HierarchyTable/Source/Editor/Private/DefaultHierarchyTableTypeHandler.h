// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableTypeHandler.h"

#include "DefaultHierarchyTableTypeHandler.generated.h"

UCLASS()
class UHierarchyTable_TableTypeHandler_Default final : public UHierarchyTable_TableTypeHandler
{
	GENERATED_BODY()

public:
	virtual void ConstructHierarchy() override;

	virtual bool FactoryConfigureProperties(FInstancedStruct& TableType) const override;
};