// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatColumn.h"

#include "Widgets/Input/SSpinBox.h"
#include "HierarchyTable.h"
#include "Editor.h"
#include "HierarchyTableDefaultTypes.h"

#define LOCTEXT_NAMESPACE "FHierarchyTableColumn_Float"

TSharedRef<SWidget> FHierarchyTableColumn_Float::CreateHeaderWidget()
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FHierarchyTableColumn_Float::CreateEntryWidget(TObjectPtr<UHierarchyTable> HierarchyTable, int32 EntryIndex)
{
	return SNew(SSpinBox<float>)
		.IsEnabled_Lambda([HierarchyTable, EntryIndex]() { return HierarchyTable->GetTableEntry(EntryIndex)->IsOverridden(); })
		.MinDesiredWidth(100.0f)
		.Value_Lambda([HierarchyTable, EntryIndex]()
			{
				return HierarchyTable->GetTableEntry(EntryIndex)->GetValue<FHierarchyTable_ElementType_Float>()->Value;
			})
		.OnValueChanged_Lambda([HierarchyTable, EntryIndex](float NewValue)
			{
				// TODO: Update API to avoid having to manually regenerate guid
				HierarchyTable->GetMutableTableEntry(EntryIndex)->GetMutableValue<FHierarchyTable_ElementType_Float>()->Value = NewValue;
				HierarchyTable->RegenerateEntriesGuid();
			})
		.OnBeginSliderMovement_Lambda([HierarchyTable]()
			{
				GEditor->BeginTransaction(LOCTEXT("SetFloatValue", "Set Float Value"));
				HierarchyTable->Modify();
			})
		.OnEndSliderMovement_Lambda([](float)
			{
				GEditor->EndTransaction();
			});
}

#undef LOCTEXT_NAMESPACE