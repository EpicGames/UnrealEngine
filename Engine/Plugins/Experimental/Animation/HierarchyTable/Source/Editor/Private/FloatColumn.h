// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHierarchyTableColumn.h"

#define LOCTEXT_NAMESPACE "HierarchyTableColumn_Float"

struct FHierarchyTableColumn_Float : public IHierarchyTableColumn
{
public:
	virtual FName GetColumnId() const override { return FName("Float"); }
	virtual FText GetColumnLabel() const override { return LOCTEXT("FloatLabel", "Float"); }
	virtual float GetColumnSize() const override { return 1.0f; }

	virtual TSharedRef<SWidget> CreateEntryWidget(TObjectPtr<UHierarchyTable> HierarchyTable, int32 EntryIndex);
	virtual TSharedRef<SWidget> CreateHeaderWidget();
};

#undef LOCTEXT_NAMESPACE