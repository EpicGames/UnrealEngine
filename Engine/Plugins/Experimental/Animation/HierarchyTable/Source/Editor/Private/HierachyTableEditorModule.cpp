// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"
#include "DefaultHierarchyTableType.h"
#include "HierarchyTableDefaultTypes.h"
#include "DefaultHierarchyTableTypeHandler.h"
#include "FloatColumn.h"
#include "SHierarchyTable.h"

void FHierarchyTableEditorModule::StartupModule()
{
	BuiltinTableTypes.Add(FHierarchyTable_TableType_Default::StaticStruct());
	RegisterTableType(FHierarchyTable_TableType_Default::StaticStruct(), UHierarchyTable_TableTypeHandler_Default::StaticClass());

	BuiltinElementTypes.Add(FHierarchyTable_ElementType_Float::StaticStruct());
	RegisterElementTypeEditorColumns(FHierarchyTable_ElementType_Float::StaticStruct(),
		{	
			MakeShared<FHierarchyTableColumn_Float>(),
		});
}

void FHierarchyTableEditorModule::ShutdownModule()
{
	for (TWeakObjectPtr<UScriptStruct> WeakPtr : BuiltinTableTypes)
	{
		if (const UScriptStruct* StructPtr = WeakPtr.Get())
		{
			UnregisterTableType(StructPtr);
		}
	}

	for (TWeakObjectPtr<UScriptStruct> WeakPtr : BuiltinElementTypes)
	{
		if (const UScriptStruct* StructPtr = WeakPtr.Get())
		{
			UnregisterElementTypeEditorColumns(StructPtr);
		}
	}
}

void FHierarchyTableEditorModule::RegisterTableType(const UScriptStruct* TableType, const UClass* Handler)
{
	TableHandlers.Add(TableType, Handler);
}

void FHierarchyTableEditorModule::UnregisterTableType(const UScriptStruct* TableType)
{
	TableHandlers.Remove(TableType);
}

const TObjectPtr<UHierarchyTable_TableTypeHandler> FHierarchyTableEditorModule::CreateTableHandler(const TObjectPtr<UHierarchyTable> HierarchyTable)
{
	const UClass** Result = TableHandlers.Find(HierarchyTable->GetTableMetadataStruct());
	if (Result)
	{
		TObjectPtr<UHierarchyTable_TableTypeHandler> Handler = NewObject<UHierarchyTable_TableTypeHandler>((UObject*)GetTransientPackage(), *Result);
		Handler->SetHierarchyTable(HierarchyTable);

		return Handler;
	}
	return nullptr;
}

const TObjectPtr<UHierarchyTable_TableTypeHandler> FHierarchyTableEditorModule::CreateTableHandler(const TObjectPtr<const UScriptStruct> TableType)
{
	const UClass** Result = TableHandlers.Find(TableType);
	if (Result)
	{
		return NewObject<UHierarchyTable_TableTypeHandler>((UObject*)GetTransientPackage(), *Result);
	}
	return nullptr;
}

void FHierarchyTableEditorModule::RegisterElementTypeEditorColumns(const UScriptStruct* ElementType, const TArray<TSharedPtr<IHierarchyTableColumn>> Columns)
{
	EditorColumns.Add(ElementType, Columns);
}

void FHierarchyTableEditorModule::UnregisterElementTypeEditorColumns(const UScriptStruct* ElementType)
{
	EditorColumns.Remove(ElementType);
}

TArray<TSharedPtr<IHierarchyTableColumn>> FHierarchyTableEditorModule::GetElementTypeEditorColumns(const TObjectPtr<const UHierarchyTable> HierarchyTable)
{
	const TArray<TSharedPtr<IHierarchyTableColumn>>* Result = EditorColumns.Find(HierarchyTable->GetElementType());

	if (Result)
	{
		return *Result;
	}

	return TArray<TSharedPtr<IHierarchyTableColumn>>();
}

TSharedRef<IHierarchyTable> FHierarchyTableEditorModule::CreateHierarchyTableWidget(const TObjectPtr<UHierarchyTable> HierarchyTable)
{
	return SNew(SHierarchyTable, HierarchyTable);
}

IMPLEMENT_MODULE(FHierarchyTableEditorModule, HierarchyTableEditor)
