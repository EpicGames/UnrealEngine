// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableAssetDefinition.h"
#include "HierarchyTable.h"
#include "HierarchyTableEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchyTableAssetDefinition)

#define LOCTEXT_NAMESPACE "HierarchyTable"

FText UAssetDefinition_HierarchyTable::GetAssetDisplayName() const
{
	return LOCTEXT("HierarchyTable", "Hierarchy Table");
}

FLinearColor UAssetDefinition_HierarchyTable::GetAssetColor() const
{
	return FLinearColor(FColor::Purple);
}

TSoftClassPtr<UObject> UAssetDefinition_HierarchyTable::GetAssetClass() const
{
	return UHierarchyTable::StaticClass();
}

EAssetCommandResult UAssetDefinition_HierarchyTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		MakeShared<FHierarchyTableEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_HierarchyTable::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Misc, LOCTEXT("AssetDefinition_HierarchyTableSubMenu", "Other"), ECategoryMenuType::Section)
		};
	return Categories;
}

#undef LOCTEXT_NAMESPACE
