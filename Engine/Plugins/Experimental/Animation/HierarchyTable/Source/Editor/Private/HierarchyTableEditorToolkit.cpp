// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableEditorToolkit.h"

#include "HierarchyTableEditorModule.h"
#include "HierarchyTableType.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SHierarchyTable.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "HierarchyTableEditorToolkit"

void FHierarchyTableEditorToolkit::InitEditor(const TArray<UObject*>& InObjects)
{
	HierarchyTable = CastChecked<UHierarchyTable>(InObjects[0]);

	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("HierarchyTableEditorLayout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab("HierarchyTableEditorTableTab", ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab("HierarchyTableEditorDetailsTab", ETabState::OpenedTab)
			)
		);

	FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, {}, "HierarchyTableEditor", Layout, true, true, InObjects);

	ExtendToolbar();
}

void FHierarchyTableEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("HierarchyTableEditor", "Hierarchy Table Editor"));

	// Table view
	{
		InTabManager->RegisterTabSpawner("HierarchyTableEditorTableTab", FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						SAssignNew(HierarchyTableWidget, SHierarchyTable, HierarchyTable)
					];
			}))
			.SetDisplayName(LOCTEXT("HierarchyTable", "Hierarchy Table"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	// Details panel
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObjects(TArray<UObject*>{ HierarchyTable });
		InTabManager->RegisterTabSpawner("HierarchyTableEditorDetailsTab", FOnSpawnTab::CreateLambda([=](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						DetailsView
					];
			}))
			.SetDisplayName(INVTEXT("Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}
}

void FHierarchyTableEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner("HierarchyTableEditorTableTab");
	InTabManager->UnregisterTabSpawner("HierarchyTableEditorDetailsTab");
}

void FHierarchyTableEditorToolkit::ExtendToolbar()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	const TObjectPtr<UHierarchyTable_TableTypeHandler> Handler = HierarchyTableModule.CreateTableHandler(HierarchyTable);
	if (!ensure(Handler))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);

	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	Handler->ExtendToolbar(ToolMenu, *HierarchyTableWidget);
}

void FHierarchyTableEditorToolkit::HandleOnEntryAdded(const int32 EntryIndex)
{
}

#undef LOCTEXT_NAMESPACE