// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableFactory.h"
#include "HierarchyTable.h"
#include "Animation/Skeleton.h"
#include "Widgets/SWindow.h"
#include "Kismet2/SClassPickerDialog.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchyTableFactory)

UHierarchyTableFactory::UHierarchyTableFactory()
{
	SupportedClass = UHierarchyTable::StaticClass();
	bCreateNew = true;
	ElementType = nullptr;
}

UObject* UHierarchyTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	TObjectPtr<UHierarchyTable> HierarchyTable = NewObject<UHierarchyTable>(InParent, Class, Name, Flags, Context);
	HierarchyTable->Initialize(TableMetadata, ElementType);

	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	const TObjectPtr<UHierarchyTable_TableTypeHandler> TableHandler = HierarchyTableModule.CreateTableHandler(HierarchyTable);
	check(TableHandler);

	TableHandler->SetHierarchyTable(HierarchyTable);
	TableHandler->ConstructHierarchy();

	return HierarchyTable;
}

bool UHierarchyTableFactory::ConfigureProperties()
{
	// Prompt the user to choose the table type
	bool bSuccess = ConfigureTableType();
	if (!bSuccess)
	{
		return false;
	}

	// Allow the table type to configure itself
	{
		FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
		const UHierarchyTable_TableTypeHandler* TableHandler = HierarchyTableModule.CreateTableHandler(TableMetadata.GetScriptStruct());
		check(TableHandler);

		bSuccess = TableHandler->FactoryConfigureProperties(TableMetadata);
		if (!bSuccess)
		{
			return false;
		}
	}

	// Prompt the user to choose the element type
	bSuccess = ConfigureElementType();
	if (!bSuccess)
	{
		return false;
	}

	return true;
}

bool UHierarchyTableFactory::ConfigureTableType()
{
	FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

	class FRuleFilter : public IStructViewerFilter
	{
	public:
		FRuleFilter()
		{
		}

		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			static const UScriptStruct* BaseStruct = FHierarchyTable_TableType::StaticStruct();
			return InStruct != BaseStruct && InStruct->IsChildOf(BaseStruct);
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}
	};

	static TSharedPtr<FRuleFilter> Filter = MakeShared<FRuleFilter>();
	FStructViewerInitializationOptions Options;
	{
		Options.StructFilter = Filter;
		Options.Mode = EStructViewerMode::StructPicker;
		Options.DisplayMode = EStructViewerDisplayMode::ListView;
		Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
		Options.bShowNoneOption = false;
		Options.bShowUnloadedStructs = false;
		Options.bAllowViewOptions = false;
	}
	
	TSharedPtr<class SWindow> PickerWindow;
	PickerWindow = SAssignNew(PickerWindow, SWindow)
		.Title(INVTEXT("Pick Type"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([&](const UScriptStruct* ChosenStruct)
				{
					FInstancedStruct InstancedStruct;
					InstancedStruct.InitializeAs(ChosenStruct);

					TableMetadata = InstancedStruct;
					PickerWindow->RequestDestroyWindow();
				}))
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());

	return TableMetadata.IsValid();
}

bool UHierarchyTableFactory::ConfigureElementType()
{
	FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

	class FRuleFilter : public IStructViewerFilter
	{
	public:
		FRuleFilter()
		{
		}

		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			static const UScriptStruct* BaseStruct = FHierarchyTable_ElementType::StaticStruct();
			return InStruct != BaseStruct && InStruct->IsChildOf(BaseStruct);
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}
	};

	static TSharedPtr<FRuleFilter> Filter = MakeShared<FRuleFilter>();
	FStructViewerInitializationOptions Options;
	{
		Options.StructFilter = Filter;
		Options.Mode = EStructViewerMode::StructPicker;
		Options.DisplayMode = EStructViewerDisplayMode::ListView;
		Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
		Options.bShowNoneOption = false;
		Options.bShowUnloadedStructs = false;
		Options.bAllowViewOptions = false;
	}
	
	TSharedPtr<class SWindow> PickerWindow;
	PickerWindow = SAssignNew(PickerWindow, SWindow)
		.Title(INVTEXT("Pick Type"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([&](const UScriptStruct* ChosenStruct)
				{
					ElementType = ChosenStruct;
					PickerWindow->RequestDestroyWindow();
				}))
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());

	return ElementType != nullptr;
}
