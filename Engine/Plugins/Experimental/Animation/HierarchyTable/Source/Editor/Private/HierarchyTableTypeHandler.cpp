// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableTypeHandler.h"
#include "HierarchyTable.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchyTableTypeHandler)

FSlateIcon UHierarchyTable_TableTypeHandler::GetEntryIcon(const int32 EntryIndex) const
{
	return FSlateIcon();
}

FSlateColor UHierarchyTable_TableTypeHandler::GetEntryIconColor(const int32 EntryIndex) const
{
	return FSlateColor::UseForeground();
}

bool UHierarchyTable_TableTypeHandler::RenameEntry(const int32 EntryIndex, const FName NewName)
{
	if (!ensure(CanRenameEntry(EntryIndex)))
	{
		return false;
	}

	if (HierarchyTable->HasIdentifier(NewName))
	{
		return false;
	}

	if (FHierarchyTableEntryData* const Entry = HierarchyTable->GetMutableTableEntry(EntryIndex))
	{
		Entry->Identifier = NewName;
	}

	return true;
}

bool UHierarchyTable_TableTypeHandler::RemoveEntry(const int32 EntryIndex)
{
	if (!ensure(CanRemoveEntry(EntryIndex)))
	{
		return false;
	}

	HierarchyTable->RemoveEntry(EntryIndex);

	return true;
}
