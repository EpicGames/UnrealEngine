// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHierarchyTable.h"

#include "HierarchyTableEditorModule.h"
#include "Framework/Commands/GenericCommands.h"
#include "HierarchyTable.h"
#include "IHierarchyTableColumn.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SHierarchyTableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SHierarchyTable"

const FName SHierarchyTable::FColumns::IdentifierId("Identifier");
const FName SHierarchyTable::FColumns::OverrideId("Override");

void SHierarchyTable::Construct(const FArguments& InArgs, TObjectPtr<UHierarchyTable> InHierarchyTable)
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

	TableHandler = HierarchyTableModule.CreateTableHandler(InHierarchyTable);

	HierarchyTable = InHierarchyTable;

	TSharedPtr<SHeaderRow> HeaderRow;

	ChildSlot
		[
			SAssignNew(TreeView, STreeView<TSharedPtr<FTreeItem>>)
				.TreeItemsSource(&RootItems)
				.OnGenerateRow(this, &SHierarchyTable::TreeView_GenerateItemRow)
				.OnGetChildren(this, &SHierarchyTable::TreeView_HandleGetChildren)
				.OnContextMenuOpening(this, &SHierarchyTable::TreeView_HandleContextMenuOpening)
				.OnItemScrolledIntoView(this, &SHierarchyTable::TreeView_OnItemScrolledIntoView)
				.HighlightParentNodesForSelection(true)
				.HeaderRow
				(
					SAssignNew(HeaderRow, SHeaderRow)
					+ SHeaderRow::Column(SHierarchyTable::FColumns::IdentifierId)
					.FillWidth(0.5f)
					.DefaultLabel(LOCTEXT("IdentifierLabel", "Identifier"))
					+ SHeaderRow::Column(SHierarchyTable::FColumns::OverrideId)
					.FixedWidth(24.0f)
					.HAlignHeader(HAlign_Center)
					.VAlignHeader(VAlign_Center)
					.HAlignCell(HAlign_Fill)
					.VAlignCell(VAlign_Fill)
					[
						SNew(SBox)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::GetBrush("DetailsView.OverrideHere"))
							]
					]
				)
		];

	TArray<TSharedPtr<IHierarchyTableColumn>> ElementTypeColumns = HierarchyTableModule.GetElementTypeEditorColumns(InHierarchyTable);

	for (TSharedPtr<IHierarchyTableColumn> Column : ElementTypeColumns)
	{
		HeaderRow->AddColumn(SHeaderRow::Column(Column->GetColumnId())
			.DefaultLabel(Column->GetColumnLabel())
			.FillWidth(Column->GetColumnSize() * 0.5f));
	}

	RegenerateTreeViewItems();

	// Expand all tree items on construction
	for (TSharedPtr<FTreeItem> TreeItem : GetAllTreeItems())
	{
		TreeView->SetItemExpansion(TreeItem, true);
	}
}

SHierarchyTable::~SHierarchyTable()
{
}

int32 SHierarchyTable::GetSelectedEntryIndex() const
{
	TArray<TSharedPtr<FTreeItem>> SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num() > 0)
	{
		return SelectedItems[0]->Index;
	}

	return INDEX_NONE;
}

void SHierarchyTable::TreeView_HandleGetChildren(TSharedPtr<FTreeItem> InItem, TArray<TSharedPtr<FTreeItem>>& OutChildren) const
{
	OutChildren.Append(InItem->Children);
}

void SHierarchyTable::TreeView_OnItemScrolledIntoView(TSharedPtr<FTreeItem> InItem, const TSharedPtr<ITableRow>& InWidget)
{
	if (DeferredRenameRequest.IsValid())
	{
		DeferredRenameRequest->OnRenameRequested.ExecuteIfBound();
		DeferredRenameRequest.Reset();
	}
}

TSharedRef<ITableRow> SHierarchyTable::TreeView_GenerateItemRow(TSharedPtr<SHierarchyTable::FTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SHierarchyTableRow, OwnerTable, SharedThis(this), Item)
		.OnRenamed_Lambda([this, Item](const FName NewName)
			{
				const FScopedTransaction Transaction(LOCTEXT("RenameEntry_Transaction", "Rename Entry"));
				HierarchyTable->Modify();

				return TableHandler->RenameEntry(Item->Index, NewName);
			});
}

TSharedPtr<SWidget> SHierarchyTable::TreeView_HandleContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<TSharedPtr<FTreeItem>> SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num() > 0)
	{
		TableHandler->ExtendContextMenu(MenuBuilder, *this);

		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameEntry_Label", "Rename"),
			LOCTEXT("RenameEntry_Tooltip", "Rename the selected entry"),
			FGenericCommands::Get().Rename->GetIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SelectedItems]()
					{
						SelectedItems[0]->OnRenameRequested.ExecuteIfBound();
					}
				),
				FCanExecuteAction::CreateLambda([this, SelectedItems]()
					{
						return TableHandler->CanRenameEntry(SelectedItems[0]->Index);
					}
				)
			));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveEntry_Label", "Remove"),
			LOCTEXT("RemoveEntry_Tooltip", "Remove the selected entry"),
			FGenericCommands::Get().Delete->GetIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SelectedItems]()
					{
						const FScopedTransaction Transaction(LOCTEXT("RemoveEntry_Transaction", "Remove Entry"));
						HierarchyTable->Modify();

						const bool bSuccess = TableHandler->RemoveEntry(SelectedItems[0]->Index);
						if (ensure(bSuccess))
						{
							RegenerateTreeViewItems();
						}
					}
				),
				FCanExecuteAction::CreateLambda([this, SelectedItems]()
					{
						return TableHandler->CanRemoveEntry(SelectedItems[0]->Index);
					}
				)
			));
	}

	return MenuBuilder.MakeWidget();
}

TArray<TSharedPtr<SHierarchyTable::FTreeItem>> SHierarchyTable::GetAllTreeItems()
{
	TArray<TSharedPtr<SHierarchyTable::FTreeItem>> AllItems;
	AllItems.Append(RootItems);

	for (int32 Index = 0; Index < AllItems.Num(); ++Index)
	{
		AllItems.Append(AllItems[Index]->Children);
	}

	return AllItems;
}

void SHierarchyTable::RegenerateTreeViewItems()
{
	// Make note of all tree items currently expanded
	TSet<FName> ExpandedAttributeNames;
	{
		for (TSharedPtr<FTreeItem> TreeItem : GetAllTreeItems())
		{
			if (TreeView->IsItemExpanded(TreeItem))
			{
				ExpandedAttributeNames.Add(TreeItem->Name);
			}
		}
	}

	// Rebuild items
	{
		RootItems.Reset();

		TMap<FName, TSharedPtr<FTreeItem>> ItemMap;

		const TArray<FHierarchyTableEntryData>& TableData = HierarchyTable->GetTableData();

		for (int32 EntryIndex = 0; EntryIndex < TableData.Num(); ++EntryIndex)
		{
			const FHierarchyTableEntryData& Entry = TableData[EntryIndex];

			TSharedPtr<FTreeItem> Item = MakeShared<FTreeItem>();
			Item->Name = Entry.Identifier;
			Item->Index = EntryIndex;

			if (Entry.Parent == INDEX_NONE)
			{
				RootItems.Add(Item);
			}
			else
			{
				const FHierarchyTableEntryData* ParentEntry = HierarchyTable->GetTableEntry(Entry.Parent);
				check(ParentEntry);

				const TSharedPtr<FTreeItem>* ParentTreeItem = ItemMap.Find(ParentEntry->Identifier);
				check(ParentTreeItem);

				(*ParentTreeItem)->Children.Add(Item);
			}

			ItemMap.Add(Item->Name, Item);
		}
	}

	// Update tree view and restore tree item expanded states
	{
		check(TreeView);
		TreeView->RebuildList();

		for (TSharedPtr<FTreeItem> TreeItem : GetAllTreeItems())
		{
			if (ExpandedAttributeNames.Contains(TreeItem->Name))
			{
				TreeView->SetItemExpansion(TreeItem, true);
			}
		}
	}
}

void SHierarchyTable::PostUndo(bool bSuccess)
{
	RegenerateTreeViewItems();
}

void SHierarchyTable::PostRedo(bool bSuccess)
{
	RegenerateTreeViewItems();
}

void SHierarchyTable::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(HierarchyTable);
	Collector.AddReferencedObject(TableHandler);
}


FString SHierarchyTable::GetReferencerName() const
{
	return FString(TEXT("SHierarchyTable"));
}

void SHierarchyTable::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (HierarchyTable->GetHierarchyGuid() != TableHierarchyGuid)
	{
		TableHierarchyGuid = HierarchyTable->GetHierarchyGuid();
		RegenerateTreeViewItems();
	}
}

#undef LOCTEXT_NAMESPACE