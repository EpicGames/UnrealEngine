// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IHierarchyTable.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "EditorUndoClient.h"

class ITableRow;
class SHierarchyTableRow;
class UHierarchyTable;
class UHierarchyTable_TableTypeHandler;

class SHierarchyTable : public IHierarchyTable, public FSelfRegisteringEditorUndoClient, public FGCObject
{
public:
	friend SHierarchyTableRow;

	SLATE_BEGIN_ARGS(SHierarchyTable) {}
	SLATE_END_ARGS()

	struct FColumns
	{
		static const FName IdentifierId;
		static const FName OverrideId;
	};

	struct FTreeItem
	{
		int32 Index;
		FName Name;
		TArray<TSharedPtr<FTreeItem>> Children;

		DECLARE_DELEGATE(FOnRenameRequested);
		FOnRenameRequested OnRenameRequested;
	};

	void Construct(const FArguments& InArgs, TObjectPtr<UHierarchyTable> InHierarchyTable);
	~SHierarchyTable();

	// SCompoundWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// IHierarchyTable
	int32 GetSelectedEntryIndex() const override;

	// FSelfRegisteringEditorUndoClient
	void PostUndo(bool bSuccess) override;
	void PostRedo(bool bSuccess) override;

	// FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:
	void TreeView_HandleGetChildren(TSharedPtr<FTreeItem> InItem, TArray<TSharedPtr<FTreeItem>>& OutChildren) const;

	void TreeView_OnItemScrolledIntoView(TSharedPtr<FTreeItem> InItem, const TSharedPtr<ITableRow>& InWidget);

	TSharedRef<ITableRow> TreeView_GenerateItemRow(TSharedPtr<FTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SWidget> TreeView_HandleContextMenuOpening();

private:
	TArray<TSharedPtr<FTreeItem>> GetAllTreeItems();

	void RegenerateTreeViewItems();

private:
	TSharedPtr<STreeView<TSharedPtr<FTreeItem>>> TreeView;

	TArray<TSharedPtr<FTreeItem>> RootItems;

	TSharedPtr<FTreeItem> DeferredRenameRequest;

	TObjectPtr<UHierarchyTable> HierarchyTable;

	TObjectPtr<UHierarchyTable_TableTypeHandler> TableHandler;

	FGuid TableHierarchyGuid;
};
