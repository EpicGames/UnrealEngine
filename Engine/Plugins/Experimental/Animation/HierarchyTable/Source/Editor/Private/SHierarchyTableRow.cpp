// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHierarchyTableRow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "HierarchyTableEditorModule.h"
#include "IHierarchyTableColumn.h"
#include "HierarchyTableTypeHandler.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SHierarchyTableRow"

FReply SHierarchyTableRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SHierarchyTableRow::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<SHierarchyTable::FTreeItem> TargetItem)
{
	TOptional<EItemDropZone> ReturnedDropZone;

	return ReturnedDropZone;
}

FReply SHierarchyTableRow::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<SHierarchyTable::FTreeItem> TargetItem)
{
	return FReply::Handled();
}

void SHierarchyTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<SHierarchyTable> InHierarchyTableWidget, TSharedPtr<SHierarchyTable::FTreeItem> InTreeItem)
{
	HierarchyTableWidget = InHierarchyTableWidget;
	TreeItem = InTreeItem;

	OnRenamed = InArgs._OnRenamed;
	OnReparented = InArgs._OnReparented;

	FSuperRowType::Construct(
		FSuperRowType::FArguments()
		.Style( FAppStyle::Get(), "TableView.AlternatingRow" )
		.OnDragDetected(this, &SHierarchyTableRow::OnDragDetected)
		.OnCanAcceptDrop(this, &SHierarchyTableRow::OnCanAcceptDrop)
		.OnAcceptDrop(this, &SHierarchyTableRow::OnAcceptDrop),
		InOwnerTableView);
}

void SHierarchyTableRow::OnCommitRename(const FText& InText, ETextCommit::Type CommitInfo)
{
	const FName OldName = TreeItem->Name;
	const FName NewName = FName(InText.ToString());

	if (OnRenamed.IsBound())
	{
		const bool bSuccess = OnRenamed.Execute(NewName);
		if (bSuccess)
		{
			TreeItem->Name = NewName;
		}
	}
}

TSharedRef<SWidget> SHierarchyTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SHierarchyTable::FColumns::IdentifierId)
	{
		TSharedPtr<SInlineEditableTextBlock> InlineWidget;

		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
					.ShouldDrawWires(true)
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f, 4.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SImage)
					.Image(HierarchyTableWidget->TableHandler->GetEntryIcon(TreeItem->Index).GetSmallIcon())
					.ColorAndOpacity(HierarchyTableWidget->TableHandler->GetEntryIconColor(TreeItem->Index))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
					.Text_Lambda([this]()
						{
							return FText::FromName(TreeItem->Name);
						})
					.OnVerifyTextChanged_Lambda([this](const FText& InNewText, FText& OutErrorMessage) -> bool
						{
							if (InNewText.IsEmpty())
							{
								OutErrorMessage = LOCTEXT("AttributeNameEmpty", "Name can't be empty.");
								return false;
							}

							const FName CurrentName = HierarchyTableWidget->HierarchyTable->GetTableEntry(TreeItem->Index)->Identifier;

							if (HierarchyTableWidget->HierarchyTable->HasIdentifier(FName(InNewText.ToString())) && !InNewText.EqualTo(FText::FromName(CurrentName)))
							{
								OutErrorMessage = LOCTEXT("AttributeNameExists", "Name already exists in the hierarchy.");
								return false;
							}

							return true;
						})
					.OnTextCommitted(this, &SHierarchyTableRow::OnCommitRename)
			];

		TreeItem->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return HorizontalBox;
	}

	if (ColumnName == SHierarchyTable::FColumns::OverrideId)
	{
		const bool bHasParent = HierarchyTableWidget->HierarchyTable->GetTableEntry(TreeItem->Index)->HasParent();

		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.IsEnabled(bHasParent)
			.OnClicked_Lambda([this]()
				{
					const FScopedTransaction Transaction(LOCTEXT("ToggleOverride", "Toggle Override"));
					HierarchyTableWidget->HierarchyTable->Modify();
					
					// TODO: Update API to avoid having to manually regenerate guid
					HierarchyTableWidget->HierarchyTable->GetMutableTableEntry(TreeItem->Index)->ToggleOverridden();
					HierarchyTableWidget->HierarchyTable->RegenerateEntriesGuid();
					
					return FReply::Handled();
				})
			.ContentPadding(2.0f)
			[
				SNew(SImage)
					.Image_Lambda([this]()
						{
							const FHierarchyTableEntryData* const EntryData = HierarchyTableWidget->HierarchyTable->GetTableEntry(TreeItem->Index);
							const bool bHasOverriddenChildren = EntryData->HasOverriddenChildren();

							if (EntryData->IsOverridden())
							{
								if (bHasOverriddenChildren)
								{
									return FAppStyle::GetBrush(TEXT("DetailsView.OverrideHereInside"));
								}
								else
								{
									return FAppStyle::GetBrush(TEXT("DetailsView.OverrideHere"));
								}
							}
							else
							{
								if (bHasOverriddenChildren)
								{
									return FAppStyle::GetBrush(TEXT("DetailsView.OverrideInside"));
								}
								else
								{
									return FAppStyle::GetBrush(TEXT("DetailsView.OverrideNone"));
								}
							}
						})
					.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	const TArray<TSharedPtr<IHierarchyTableColumn>> Columns = HierarchyTableModule.GetElementTypeEditorColumns(HierarchyTableWidget->HierarchyTable);

	for (TSharedPtr<IHierarchyTableColumn> Column : Columns)
	{
		if (ColumnName == Column->GetColumnId())
		{
			return Column->CreateEntryWidget(HierarchyTableWidget->HierarchyTable, TreeItem->Index);
		}
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE