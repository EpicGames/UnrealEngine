// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SHierarchyTable.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class SHierarchyTableRow : public SMultiColumnTableRow<TSharedPtr<SHierarchyTable::FTreeItem>>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool /* Success */, FOnRenamed, FName /* New Name */);
	DECLARE_DELEGATE_TwoParams(FOnReparented, FName /* Attribute Name */, FName /* New Parent Name */);

	SLATE_BEGIN_ARGS(SHierarchyTableRow) {}
		SLATE_EVENT(FOnRenamed, OnRenamed);
		SLATE_EVENT(FOnReparented, OnReparented);
	SLATE_END_ARGS()

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<SHierarchyTable::FTreeItem> TargetItem);

	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<SHierarchyTable::FTreeItem> TargetItem);

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<SHierarchyTable> InHierarchyTableWidget, TSharedPtr<SHierarchyTable::FTreeItem> InTreeItem);

	void OnCommitRename(const FText& InText, ETextCommit::Type CommitInfo);

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<SHierarchyTable> HierarchyTableWidget;
	TSharedPtr<SHierarchyTable::FTreeItem> TreeItem;
	FOnRenamed OnRenamed;
	FOnReparented OnReparented;
};