// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableTypeHandler.h"
#include "HierarchyTableType.h"
#include "Modules/ModuleInterface.h"
#include "HierarchyTable.h"
#include "UObject/ObjectKey.h"

struct IHierarchyTableColumn;
class IHierarchyTable;

class FHierarchyTableEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	
	virtual void ShutdownModule() override;

	HIERARCHYTABLEEDITOR_API TSharedRef<IHierarchyTable> CreateHierarchyTableWidget(const TObjectPtr<UHierarchyTable> HierarchyTable);

	HIERARCHYTABLEEDITOR_API void RegisterTableType(const UScriptStruct* TableType, const UClass* Handler);

	HIERARCHYTABLEEDITOR_API void UnregisterTableType(const UScriptStruct* TableType);

	HIERARCHYTABLEEDITOR_API const TObjectPtr<UHierarchyTable_TableTypeHandler> CreateTableHandler(const TObjectPtr<UHierarchyTable> HierarchyTable);

	HIERARCHYTABLEEDITOR_API const TObjectPtr<UHierarchyTable_TableTypeHandler> CreateTableHandler(const TObjectPtr<const UScriptStruct> TableType);

	HIERARCHYTABLEEDITOR_API void RegisterElementTypeEditorColumns(const UScriptStruct* ElementType, const TArray<TSharedPtr<IHierarchyTableColumn>> Columns);

	HIERARCHYTABLEEDITOR_API void UnregisterElementTypeEditorColumns(const UScriptStruct* ElementType);

	TArray<TSharedPtr<IHierarchyTableColumn>> GetElementTypeEditorColumns(const TObjectPtr<const UHierarchyTable> HierarchyTable);

private:
	TArray<TWeakObjectPtr<UScriptStruct>> BuiltinTableTypes;

	TArray<TWeakObjectPtr<UScriptStruct>> BuiltinElementTypes;

	TMap<TObjectKey<UScriptStruct>, const UClass*> TableHandlers;

	TMap<TObjectKey<UScriptStruct>, TArray<TSharedPtr<IHierarchyTableColumn>>> EditorColumns;
};