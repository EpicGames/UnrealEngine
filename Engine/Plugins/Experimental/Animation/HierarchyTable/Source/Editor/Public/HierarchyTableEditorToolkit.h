// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HierarchyTable.h"
#include "Toolkits/AssetEditorToolkit.h"

class SHierarchyTable;

class FHierarchyTableEditorToolkit : public FAssetEditorToolkit
{
public:
	void InitEditor(const TArray<UObject*>& InObjects);

	// Begin FAssetEditorToolkit
	void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	FName GetToolkitFName() const override { return "HierarchyTableEditor"; }
	FText GetBaseToolkitName() const override { return INVTEXT("Hierarchy Table Editor"); }
	FString GetWorldCentricTabPrefix() const override { return "Hierarchy Table "; }
	FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
	// End FAssetEditorToolkit

	void ExtendToolbar();
private:
	void HandleOnEntryAdded(const int32 EntryIndex);

	TObjectPtr<UHierarchyTable> HierarchyTable;

	TSharedPtr<SHierarchyTable> HierarchyTableWidget;
};
