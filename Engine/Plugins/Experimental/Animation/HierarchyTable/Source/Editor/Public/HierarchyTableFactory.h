// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "HierarchyTableType.h"
#include "StructUtils/InstancedStruct.h"

#include "HierarchyTableFactory.generated.h"

class UScriptStruct;

UCLASS()
class UHierarchyTableFactory : public UFactory
{
	GENERATED_BODY()

public:
	UHierarchyTableFactory();

	// UFactory interface
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;

private:
	bool ConfigureTableType();

	bool ConfigureElementType();

	FInstancedStruct TableMetadata;

	const UScriptStruct* ElementType;
};
