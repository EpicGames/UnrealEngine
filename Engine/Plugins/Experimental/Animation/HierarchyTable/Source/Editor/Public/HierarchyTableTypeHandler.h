// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "StructUtils/InstancedStruct.h"
#include "HierarchyTableType.h"
#include "IHierarchyTable.h"
#include "HierarchyTableTypeHandler.generated.h"

#define UE_API HIERARCHYTABLEEDITOR_API

class UHierarchyTable;
class UScriptStruct;
class UToolMenu;
class SWidget;
class FMenuBuilder;
struct FSlateIcon;
struct FSlateColor;

UCLASS(MinimalAPI, Abstract)
class UHierarchyTable_TableTypeHandler : public UObject
{
	GENERATED_BODY()

public:
	void SetHierarchyTable(TObjectPtr<UHierarchyTable> InHierarchyTable) { HierarchyTable = InHierarchyTable; }

	/** Override to provide additional configurable properties when creating a new hierarchy table of this table type */
	virtual bool FactoryConfigureProperties(FInstancedStruct& TableType) const { return false; }

	/** Override to add additional buttons to the editor toolbar */
	virtual void ExtendToolbar(UToolMenu* ToolMenu, IHierarchyTable& HierarchyTableView) const {}

	/** Override to customize the context menu constructed for a particular entry in the table */
	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, IHierarchyTable& HierarchyTableView) const {}

	/** Override to provide an icon for a particular entry in the table */
	UE_API virtual FSlateIcon GetEntryIcon(const int32 EntryIndex) const;
	
	/** Override to provide an icon color for a particular entry in the table */
	UE_API virtual FSlateColor GetEntryIconColor(const int32 EntryIndex) const;

	/** Resets and builds the hierarchy table data according to the current table metadata */
	virtual void ConstructHierarchy() {};

	virtual bool CanRenameEntry(const int32 EntryIndex) const { return false; }
	UE_API virtual bool RenameEntry(const int32 EntryIndex, const FName NewName);

	virtual bool CanRemoveEntry(const int32 EntryIndex) const { return false; }
	UE_API virtual bool RemoveEntry(const int32 EntryIndex);

protected:
	TObjectPtr<UHierarchyTable> HierarchyTable;
};

#undef UE_API
