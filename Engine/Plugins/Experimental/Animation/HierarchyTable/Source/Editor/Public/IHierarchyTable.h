// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class IHierarchyTable : public SCompoundWidget
{
public:
	virtual int32 GetSelectedEntryIndex() const = 0;
};