// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

class UHierarchyTable;

struct IHierarchyTableColumn
{
public:
	virtual FName GetColumnId() const = 0;
	virtual FText GetColumnLabel() const = 0;
	virtual float GetColumnSize() const = 0;

	virtual TSharedRef<SWidget> CreateEntryWidget(TObjectPtr<UHierarchyTable> HierarchyTable, int32 EntryIndex) = 0;

	virtual TSharedRef<SWidget> CreateHeaderWidget() = 0;
};
