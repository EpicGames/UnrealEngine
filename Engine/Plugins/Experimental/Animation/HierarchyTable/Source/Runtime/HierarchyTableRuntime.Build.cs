// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HierarchyTableRuntime : ModuleRules
{
	public HierarchyTableRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "HTRun";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
            {
			}
		);
	}
}
