// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableRuntimeModule.h"
#include "Modules/ModuleManager.h"

void FHierarchyTableRuntimeModule::StartupModule()
{
}

void FHierarchyTableRuntimeModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHierarchyTableRuntimeModule, HierarchyTableRuntime)
