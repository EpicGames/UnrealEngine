// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTable.h"
#include "Animation/Skeleton.h"
#include "ReferenceSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchyTable)

UHierarchyTable::UHierarchyTable()
{
}

bool FHierarchyTableEntryData::HasOverriddenChildren() const
{
	return IsOverriddenOrHasOverriddenChildren(false);
}

bool FHierarchyTableEntryData::IsOverriddenOrHasOverriddenChildren(const bool bIncludeSelf) const
{
	if (bIncludeSelf && IsOverridden())
	{
		return true;
	}

	TArray<const FHierarchyTableEntryData*> Children = OwnerTable->GetChildren(*this);
	for (const FHierarchyTableEntryData* Child : Children)
	{
		if (Child->IsOverriddenOrHasOverriddenChildren(true))
		{
			return true;
		}
	}

	return false;
}

void FHierarchyTableEntryData::ToggleOverridden()
{
	if (Payload.IsSet())
	{
		Payload.Reset();
	}
	else
	{
		Payload = *GetFromClosestAncestor();
	}
}

const TOptional<FInstancedStruct>& FHierarchyTableEntryData::GetPayload() const
{
	return Payload;
}

const FInstancedStruct* FHierarchyTableEntryData::GetActualValue() const
{
	return IsOverridden() ? Payload.GetPtrOrNull() : GetFromClosestAncestor();
}

const FHierarchyTableEntryData* FHierarchyTableEntryData::GetClosestAncestor() const
{
	return IsOverridden() ? this : OwnerTable->GetTableEntry(Parent)->GetClosestAncestor();
}

const FInstancedStruct* FHierarchyTableEntryData::GetFromClosestAncestor() const
{
	check(Parent != INDEX_NONE);

	const FHierarchyTableEntryData* const ParentEntry = OwnerTable->GetTableEntry(Parent);
	return ParentEntry->GetActualValue();
}

TArray<const FHierarchyTableEntryData*> UHierarchyTable::GetChildren(const FHierarchyTableEntryData& Parent) const
{
	int32 ParentIndex = TableData.IndexOfByPredicate([Parent](const FHierarchyTableEntryData& Candidate)
		{
			return Candidate.Identifier == Parent.Identifier;
		});

	if (ParentIndex == INDEX_NONE)
	{
		return TArray<const FHierarchyTableEntryData*>();
	}

	TArray<const FHierarchyTableEntryData*> Children;

	for (const FHierarchyTableEntryData& Entry : TableData)
	{
		if (Entry.Parent == ParentIndex)
		{
			Children.Add(&Entry);
		}
	}

	return Children;
}

bool UHierarchyTable::HasIdentifier(const FName Identifier) const
{
	return TableData.ContainsByPredicate([Identifier](const FHierarchyTableEntryData& Entry)
		{
			return Entry.Identifier == Identifier;
		});
}

int32 UHierarchyTable::AddEntry(const FHierarchyTableEntryData& Entry)
{
	// Do not allow entries with duplicate identifiers
	if (HasIdentifier(Entry.Identifier))
	{
		return INDEX_NONE;
	}

	const int32 EntryIndex = TableData.Add(Entry);
	RegenerateHierarchyGuid();

	return EntryIndex;
}

void UHierarchyTable::AddBulkEntries(const TConstArrayView<FHierarchyTableEntryData> Entries)
{
	for (const FHierarchyTableEntryData& Entry : Entries)
	{
		if (!HasIdentifier(Entry.Identifier))
		{
			TableData.Add(Entry);
		}
	}

	RegenerateHierarchyGuid();
}

void UHierarchyTable::RemoveEntry(const int32 IndexToRemove)
{
	if (!TableData.IsValidIndex(IndexToRemove))
	{
		return;
	}
	FHierarchyTableEntryData& EntryToRemove = TableData[IndexToRemove];

	for (int32 EntryIndex = 0; EntryIndex < TableData.Num(); ++EntryIndex)
	{
		FHierarchyTableEntryData& Entry = TableData[EntryIndex];

		if (Entry.Parent == IndexToRemove)
		{
			Entry.Parent = EntryToRemove.Parent;
		}
		else if (Entry.Parent > IndexToRemove)
		{
			--Entry.Parent;
		}
	}

	TableData.RemoveAt(IndexToRemove);
	RegenerateHierarchyGuid();
}

FInstancedStruct UHierarchyTable::CreateDefaultValue() const
{
	FInstancedStruct OutStruct;
	OutStruct.InitializeAs(ElementType);

	return OutStruct;
}

const FHierarchyTableEntryData* const UHierarchyTable::GetTableEntry(const FName EntryIdentifier) const
{
	for (const FHierarchyTableEntryData& Entry : TableData)
	{
		if (Entry.Identifier == EntryIdentifier)
		{
			return &Entry;
		}
	}

	return nullptr;
}

const FHierarchyTableEntryData* const UHierarchyTable::GetTableEntry(const int32 EntryIndex) const
{
	if (TableData.IsValidIndex(EntryIndex))
	{
		return &TableData[EntryIndex];
	}
	return nullptr;
}

FHierarchyTableEntryData* const UHierarchyTable::GetMutableTableEntry(const int32 EntryIndex)
{
	if (TableData.IsValidIndex(EntryIndex))
	{
		return &TableData[EntryIndex];
	}
	return nullptr;
}

void UHierarchyTable::EmptyTable()
{
	TableData.Empty();
	RegenerateHierarchyGuid();
}

FGuid UHierarchyTable::GetHierarchyGuid()
{
	return HierarchyGuid;
}

void UHierarchyTable::RegenerateHierarchyGuid()
{
	HierarchyGuid = FGuid::NewGuid();
}

int32 UHierarchyTable::GetTableEntryIndex(const FName EntryIdentifier) const
{
	for (int32 EntryIndex = 0; EntryIndex < TableData.Num(); ++EntryIndex)
	{
		if (TableData[EntryIndex].Identifier == EntryIdentifier)
		{
			return EntryIndex;
		}
	}

	return INDEX_NONE;
}

#if WITH_EDITOR
FGuid UHierarchyTable::GetEntriesGuid()
{
	return EntriesGuid;
}

void UHierarchyTable::RegenerateEntriesGuid()
{
	EntriesGuid = FGuid::NewGuid(); 
}
#endif //WITH_EDITOR