// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableType.h"

#include "DefaultHierarchyTableType.generated.h"

USTRUCT(DisplayName = "Default")
struct FHierarchyTable_TableType_Default final : public FHierarchyTable_TableType
{
	GENERATED_BODY()
};