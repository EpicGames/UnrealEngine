// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HierarchyTableType.h"
#include "StructUtils/InstancedStruct.h"

#include "HierarchyTable.generated.h"

#define UE_API HIERARCHYTABLERUNTIME_API

class USkeleton;
class UHierarchyTable;

/**
 * The data associated with each item in a hierarchy table.
 */
USTRUCT()
struct FHierarchyTableEntryData
{
	GENERATED_BODY()

public:
	FHierarchyTableEntryData()
		: Parent(INDEX_NONE)
	{
	}

	friend UHierarchyTable;

	// The hierarchy table that this entry resides in.
	UPROPERTY()
	TObjectPtr<UHierarchyTable> OwnerTable;

	/**
	 * The actual user set data this element stores, matches the owner table's element type.
	 * If unset, will inherit from the parent.
	 * Is always set for entries with no parent.
	 */
	UPROPERTY()
	TOptional<FInstancedStruct> Payload;

	// This entry's display name and unique identifier.
	UPROPERTY()
	FName Identifier;

	// The index of this entry's parent, is INDEX_NONE in the case of a root element.
	UPROPERTY()
	int32 Parent;

	// A second data payload specific to the table's type and is read-only metadata.
	UPROPERTY()
	FInstancedStruct TablePayload;

public:
	bool HasParent() const { return Parent != INDEX_NONE; }
	bool IsOverridden() const { return Payload.IsSet(); }
	UE_API bool HasOverriddenChildren() const;

	UE_API void ToggleOverridden();

	// Assumes payload is set, i.e. is overridden
	UE_API const TOptional<FInstancedStruct>& GetPayload() const;

	template <typename T>
	const T* GetValue() const
	{
		const FInstancedStruct* StructPtr = GetActualValue();
		check(StructPtr->IsValid());
		return StructPtr->GetPtr<T>();
	}

	template <typename T>
	const T& GetMetadata() const
	{
		return TablePayload.Get<T>();
	}

	template <typename T>
	T* GetMutableValue()
	{
		check(IsOverridden());
		return Payload.GetValue().GetMutablePtr<T>();
	}

	UE_API const FHierarchyTableEntryData* GetClosestAncestor() const;

private:
	// TODO: This needs to be cached in some way as it potentially goes up the entire hierarchy until it finds an ancestor with an overridden value
	// This is called each time a widget is ticked so grows exponentially with the height of the hierarchy. This needs addressing at some point
	// but cached value must be updated when any of its direct ancestors' value is updated.
	UE_API const FInstancedStruct* GetActualValue() const;

	UE_API const FInstancedStruct* GetFromClosestAncestor() const;

	UE_API bool IsOverriddenOrHasOverriddenChildren(const bool bIncludeSelf) const;
};

/**
 * A general-purpose container asset for storing typed hierarchical data.
 */
UCLASS(MinimalAPI, EditInlineNew, BlueprintType)
class UHierarchyTable : public UObject
{
	GENERATED_BODY()

public:
	UE_API UHierarchyTable();

private:
	// The table metadata stores any data dependent on the table type needed to create and maintain the hierarchy.
	UPROPERTY(EditAnywhere, Category = Default)
	FInstancedStruct TableMetadata;

	// The element type is the type that each element is mapped into.
	UPROPERTY(VisibleAnywhere, Category = Default)
	TObjectPtr<const UScriptStruct> ElementType;

	// The actual table data
	UPROPERTY()
	TArray<FHierarchyTableEntryData> TableData;

public:
	template <typename InTableType>
	bool IsTableType() const
	{
		return TableMetadata.GetScriptStruct() == InTableType::StaticStruct();
	}

	template <typename InTableType>
	InTableType GetTableMetadata() const
	{
		check(IsTableType<InTableType>());
		return TableMetadata.Get<InTableType>();
	}

	const UScriptStruct* GetTableMetadataStruct() const
	{
		return TableMetadata.GetScriptStruct();
	}
	
	template <typename InElementType>
	bool IsElementType() const
	{
		return ElementType == InElementType::StaticStruct();
	}

	UE_API FInstancedStruct CreateDefaultValue() const;

	void Initialize(const FInstancedStruct& InTableMetadata, const TObjectPtr<const UScriptStruct> InElementType)
	{
		TableMetadata = InTableMetadata;
		ElementType = InElementType;
		TableData.Reset();
	}

	const FInstancedStruct& GetTableMetadata() const { return TableMetadata; }

	const TObjectPtr<const UScriptStruct>& GetElementType() const { return ElementType; }

	const TArray<FHierarchyTableEntryData>& GetTableData() const { return TableData; }

	UE_API void EmptyTable();

	// TODO: Remove in the future to avoid API signatures using indices
	UE_API int32 GetTableEntryIndex(const FName EntryIdentifier) const;

	UE_API const FHierarchyTableEntryData* const GetTableEntry(const FName EntryIdentifier) const;

	// TODO: Remove in the future to avoid API signatures using indices
	UE_API const FHierarchyTableEntryData* const GetTableEntry(const int32 EntryIndex) const;

	// TODO: Remove in the future to avoid API signatures using indices
	UE_API FHierarchyTableEntryData* const GetMutableTableEntry(const int32 EntryIndex);

	UE_API int32 AddEntry(const FHierarchyTableEntryData& Entry);

	UE_API void AddBulkEntries(const TConstArrayView<FHierarchyTableEntryData> Entries);

	UE_API void RemoveEntry(const int32 Index);

	UE_API TArray<const FHierarchyTableEntryData*> GetChildren(const FHierarchyTableEntryData& Parent) const;

	UE_API bool HasIdentifier(const FName Identifier) const;

	template <typename T>
	FHierarchyTableEntryData* FindEntry(const FName EntryIdentifier)
	{
		return TableData.FindByPredicate([EntryIdentifier](const FHierarchyTableEntryData& Entry)
		{
			return Entry.Identifier == EntryIdentifier;
		});
	}

	UE_API FGuid GetHierarchyGuid();

#if WITH_EDITOR
	// Returns a GUID encoding the state of the values of all entries
	// i.e. this GUID is invalidated when values inside the table are modified
	UE_API FGuid GetEntriesGuid();
	
	// TODO: Make private in subsequent API changes
	UE_API void RegenerateEntriesGuid();
#endif

private:
	UE_API void RegenerateHierarchyGuid();

	UPROPERTY()
	FGuid HierarchyGuid;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
    FGuid EntriesGuid;
#endif
};

#undef UE_API
