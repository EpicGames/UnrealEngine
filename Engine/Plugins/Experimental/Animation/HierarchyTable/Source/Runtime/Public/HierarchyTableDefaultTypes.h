// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableType.h"

#include "HierarchyTableDefaultTypes.generated.h"

USTRUCT(DisplayName = "Float")
struct FHierarchyTable_ElementType_Float final : public FHierarchyTable_ElementType
{
	GENERATED_BODY()

	FHierarchyTable_ElementType_Float()
		: Value(0.0)
	{
	}

	UPROPERTY()
	float Value;
};