// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableType.generated.h"

USTRUCT()
struct FHierarchyTable_TableType
{
	GENERATED_BODY()
};

USTRUCT()
struct FHierarchyTable_TablePayloadType
{
	GENERATED_BODY()
};

USTRUCT()
struct FHierarchyTable_ElementType
{
	GENERATED_BODY()
};
